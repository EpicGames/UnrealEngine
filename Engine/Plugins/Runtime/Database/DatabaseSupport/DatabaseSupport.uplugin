{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Database Support",
	"Description": "Abstract Database Support",
	"Category": "Database",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"Hidden": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "DatabaseSupport",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}