// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**  
 * Enums for Database types.  Each Database has their own set of DB types and  
 */  
enum EDataBaseUnrealTypes  
{  
	DBT_UNKOWN,  
	DBT_FLOAT,  
	DBT_INT,  
	DBT_STRING,  
};  

/**   
 * This struct holds info relating to a column.  Specifically, we need to get back  
 * certain meta info from a RecordSet so we can "Get" data from it.  
 */  
struct FDatabaseColumnInfo  
{  
	/** Default constructor **/  
	FDatabaseColumnInfo(): DataType(DBT_UNKOWN) {}  

	/** The name of the column **/  
	FString ColumnName;  

	/** This is the type of data in this column.  (e.g. so you can do GetFloat or GetInt on the column **/  
	EDataBaseUnrealTypes DataType;  


	bool operator==(const FDatabaseColumnInfo& OtherFDatabaseColumnInfo) const  
	{  
		return (ColumnName==OtherFDatabaseColumnInfo.ColumnName) && (DataType==OtherFDatabaseColumnInfo.DataType);  
	}  

};   

/**
 * Empty base class for iterating over database records returned via query. Used on platforms not supporting
 * a direct database connection.
 */
class FDataBaseRecordSet
{
	// Protected functions used internally for iteration.

protected:
	/** Moves to the first record in the set. */
	virtual void MoveToFirst()
	{}
	/** Moves to the next record in the set. */
	virtual void MoveToNext()
	{}
	/**
	 * Returns whether we are at the end.
	 *
	 * @return true if at the end, false otherwise
	 */
	virtual bool IsAtEnd() const
	{
		return true;
	}

public:

	/**
	 * @return True if the record set has an error set.
	 */
	virtual bool HasError() const
	{
		return false;
	}
	/**
	 * @return The raw error code that was set, if any (@see HasError).
	 */
	virtual int32 GetErrorCode() const
	{
		return 0;
	}
	/**
	 * @return The error message that was set, if any (@see HasError).
	 */
	virtual FString GetErrorMessage() const
	{
		return FString();
	}

	/** 
	 *   Returns a count of the number of records in the record set
	 */
	virtual int32 GetRecordCount() const
	{ 
		return 0;
	}

	/**
	 * Returns a string associated with the passed in field/ column for the current row.
	 *
	 * @param	Column	Name of column to retrieve data for in current row
	 */
	virtual FString GetString( const TCHAR* Column ) const
	{
		return TEXT("No database connection compiled in.");
	}

	/**
	 * Returns an integer associated with the passed in field/ column for the current row.
	 *
	 * @param	Column	Name of column to retrieve data for in current row
	 */
	virtual int32 GetInt( const TCHAR* Column ) const
	{
		return 0;
	}

	/**
	 * Returns a float associated with the passed in field/ column for the current row.
	 *
	 * @param	Column	Name of column to retrieve data for in current row
	 */
	virtual float GetFloat( const TCHAR* Column ) const
	{
		return 0;
	}

	/**
	 * Returns a int64 associated with the passed in field/ column for the current row.
	 *
	 * @param	Column	Name of column to retrieve data for in current row
	 */
	virtual int64 GetBigInt( const TCHAR* Column ) const
	{
		return 0;
	}

	/**  
      * Returns the set of column names for this Recordset.  This is useful for determining  
      * what you can actually ask the record set for without having to hard code those ahead of time.  
      */  
     virtual TArray<FDatabaseColumnInfo> GetColumnNames() const  
     {  
          TArray<FDatabaseColumnInfo> Retval;  
          return Retval;  
     }

	/** Virtual destructor as class has virtual functions. */
	virtual ~FDataBaseRecordSet()
	{}

	/**
	 * Iterator helper class based on FObjectIterator.
	 */
	class TIterator
	{
	public:
		/** 
		 * Initialization constructor.
		 *
		 * @param	InRecordSet		RecordSet to iterate over
		 */
		TIterator( FDataBaseRecordSet* InRecordSet )
		: RecordSet( InRecordSet )
		{
			RecordSet->MoveToFirst();
		}

		/** 
		 * operator++ used to iterate to next element.
		 */
		void operator++()
		{ 
			RecordSet->MoveToNext();	
		}

		/** Conversion to "bool" returning true if the iterator is valid. */
		inline explicit operator bool() const
		{ 
			return !RecordSet->IsAtEnd(); 
		}
		/** inverse of the "bool" operator */
		inline bool operator !() const 
		{
			return !(bool)*this;
		}

		// Access operators
		inline FDataBaseRecordSet* operator*() const
		{
			return RecordSet;
		}
		inline FDataBaseRecordSet* operator->() const
		{
			return RecordSet;
		}

	protected:
		/** Database record set being iterated over. */
		FDataBaseRecordSet* RecordSet;
	};
};

/**
 * Empty base class for database access via executing SQL commands. Used on platforms not supporting
 * a direct database connection.
 */
class FDataBaseConnection
{
public:
	/** Virtual destructor as we have virtual functions. */
	virtual ~FDataBaseConnection() 
	{}

	/**
	 * Opens a connection to the database.
	 *
	 * @param	ConnectionString	Connection string passed to database layer
	 * @param   RemoteConnectionIP  The IP address which the RemoteConnection should connect to
	 * @param   RemoteConnectionStringOverride  The connection string which the RemoteConnection is going to utilize
	 *
	 * @return	true if connection was successfully established, false otherwise
	 */
	virtual bool Open( const TCHAR* ConnectionString, const TCHAR* RemoteConnectionIP, const TCHAR* RemoteConnectionStringOverride )
	{
		return false;
	}
	
	/**
	 * Closes connection to database.
	 */
	virtual void Close()
	{}

	/**
	 * Executes the passed in command on the database.
	 *
	 * @param CommandString		Command to execute
	 *
	 * @return true if execution was successful, false otherwise
	 */
	virtual bool Execute( const TCHAR* CommandString )
	{
		return false;
	}

	/**
	 * Executes the passed in command on the database. The caller is responsible for deleting
	 * the created RecordSet.
	 *
	 * @param CommandString		Command to execute
	 * @param RecordSet			Reference to recordset pointer that is going to hold result
	 *
	 * @return true if execution was successful, false otherwise
	 */
	virtual bool Execute( const TCHAR* CommandString, FDataBaseRecordSet*& RecordSet )
	{
		RecordSet = nullptr;
		return false;
	}

	/**
	 * Executes the passed in command on the database. The caller is responsible for deleting
	 * the created RecordSet.
	 *
	 * @param CommandString		Command to execute
	 * @param RecordSet			Reference to recordset pointer that is going to hold result
	 *
	 * @return true if execution was successful, false otherwise
	 */
	inline bool Execute(const TCHAR* CommandString, TUniquePtr< FDataBaseRecordSet >& RecordSet)
	{
		FDataBaseRecordSet* RecordSetRaw = nullptr;
		bool bRet = Execute(CommandString, RecordSetRaw);
		RecordSet.Reset(RecordSetRaw);
		return bRet;
	}
};
